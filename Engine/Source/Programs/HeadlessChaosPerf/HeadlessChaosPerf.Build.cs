// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

// HeadlessChaosPerf is a stand-alone application used for low-level perf testing of the Chaos Engine.
// It does not use UE Engine, but runs on any platform.
// Perf testing results are reported use CSVPerf
public class HeadlessChaosPerf : ModuleRules
{
	public HeadlessChaosPerf(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.Add("Launch");

		SetupModulePhysicsSupport(Target);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "ApplicationCore",
				"Core",
				"CoreUObject",
				"Projects",
				"GeometryCore",
				"ChaosVehiclesCore",
				"GoogleTest",
			}
        );

		PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");

		// CSV_PROFILER is tied to WITH_ENGINE in CsvProfiler.h, but we want it without engine
		PrivateDefinitions.Add("CSV_PROFILER=1");
	}
}
