// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.Collections.Generic;

[SupportedPlatforms(UnrealPlatformClass.All)]
public class HeadlessChaosPerfTarget : TargetRules
{
	public HeadlessChaosPerfTarget(TargetInfo Target) : base(Target)
	{
		Type = TargetType.Program;
		LinkType = TargetLinkType.Monolithic;

        ExeBinariesSubFolder = "NotForLicensees";
		LaunchModuleName = "HeadlessChaosPerf";

		bBuildDeveloperTools = false;

		// HeadlessChaosPerf doesn't ever compile with the engine linked in
		bCompileAgainstEngine = false;
		bCompileAgainstCoreUObject = true;

        bHasExports = false;

        bUseLoggingInShipping = true;

        // Console application, not a Windows app (sets entry point to main(), instead of WinMain())
        bIsBuildingConsoleApplication = true;

		GlobalDefinitions.Add("CHAOS_SERIALIZE_OUT=1");
		GlobalDefinitions.Add("CSV_PROFILER=1");
	}
}
