// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ProfilingDebugging/CSVProfiler.h"

CSV_DEFINE_CATEGORY(ChaosPerf, true);
