// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosPerf/ChaosPerf.h"
#include "Misc/Paths.h"

namespace ChaosPerf
{
	//
	// Base class for a one-shot perf test (i.e., no frame subdivisions in the output data)
	//
	class FPerfTest
	{
	public:
		virtual ~FPerfTest()
		{
		}

	protected:
		FPerfTest(const FString& InTestName)
		{
			TestName = InTestName;
			CSVProfiler = nullptr;
		}

		// Override to do setup outside of the timing capture
		virtual void CreateTest() {}

		virtual void RunTest() {}

		// Override to do teardown outside of the timing capture
		virtual void DestroyTest() {}

	private:
		friend class FPerfTestRegistry;

		void Run()
		{
			SetUp();
			RunTest();
			TearDown();
		}

		void SetUp()
		{
			FilenamePrefix = TEXT("Profile ") + TestName;
			Filename = FString::Printf(TEXT("%s(%s).csv"), *FilenamePrefix, *FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")));
			RootFolder = FPaths::ProfilingDir() + TEXT("Metrics");

			StartCapture();

			FCsvProfiler::BeginStat("Total", CSV_CATEGORY_INDEX(ChaosPerf));

			{
				CSV_SCOPED_TIMING_STAT(ChaosPerf, Setup);
				CreateTest();
			}

			FCsvProfiler::BeginStat("Test", CSV_CATEGORY_INDEX(ChaosPerf));
		}

		void TearDown()
		{
			FCsvProfiler::EndStat("Test", CSV_CATEGORY_INDEX(ChaosPerf));

			{
				CSV_SCOPED_TIMING_STAT(ChaosPerf, Shutdown);
				DestroyTest();
			}

			FCsvProfiler::EndStat("Total", CSV_CATEGORY_INDEX(ChaosPerf));

			StopCapture();
		}

		// CSV Profiler is intended to be used for frame-based profiling with a Game Thread, Render Thread and possibly other threads.
		// We need to do some Begin/EndFrame shenanigans to fool it into working for one-shot tests.
		// NOTE: Most methods on CSV Profiler put evbenmts into a queue and that queue is only processed in EndFrame()
		void StartCapture()
		{
			CSVProfiler = FCsvProfiler::Get();

			CSVProfiler->BeginFrame();
			CSVProfiler->BeginCapture(-1, RootFolder, Filename);
			CSVProfiler->EndFrame();

			CSVProfiler->BeginFrame();
		}

		void StopCapture()
		{

			// EndCapture event is processed in EndFrame so must come first
			CSVProfiler->EndCapture();
			CSVProfiler->EndFrame();

			// Must run one more frame to "wait for render thread" (we don't have one but this works)
			CSVProfiler->BeginFrame();
			CSVProfiler->EndFrame();

			CSVProfiler = nullptr;
		}

		FCsvProfiler* CSVProfiler;
		FString TestName;
		FString FilenamePrefix;
		FString Filename;
		FString RootFolder;
	};


	// A registry of all the tests in the program
	class FPerfTestRegistry
	{
	public:
		static FPerfTestRegistry& Get()
		{
			static FPerfTestRegistry SRegistry;
			return SRegistry;
		}

		void Add(const TSharedPtr<FPerfTest>& Test)
		{
			Tests.Add(Test);
		}

		void RunAll()
		{
			for (const auto& Test : Tests)
			{
				Test->Run();
			}
		}

	private:
		TArray<TSharedPtr<FPerfTest>> Tests;
	};


	// A utlity object for registering tests with the registry
	class FPerfTestRegistrar
	{
	public:
		FPerfTestRegistrar(const TSharedPtr<FPerfTest>& Test)
		{
			FPerfTestRegistry::Get().Add(Test);
		}
	};

	#define CHAOSPERF_TEST_BASEIMPL(BASE_CLASS, TEST_CLASS, TEST_NAME) \
		class TEST_CLASS : public BASE_CLASS	\
		{ \
		public: \
			TEST_CLASS() : FPerfTest(#TEST_NAME) {} \
			virtual void RunTest() override final; \
		}; \
		TSharedPtr<TEST_CLASS> Test_ ## TEST_NAME = MakeShared<TEST_CLASS>(); \
		FPerfTestRegistrar TestRegistrar_ ## TEST_NAME(Test_ ## TEST_NAME); \
		void TEST_CLASS::RunTest()


	#define CHAOSPERF_TEST_BASE(BASE_CLASS, TEST_NAME) CHAOSPERF_TEST_BASEIMPL(BASE_CLASS, FPerfTest_ ## TEST_NAME, TEST_NAME)


	// Simple test macro for tests which require no setup (or the setup cost is negligible)
	// E.g.,
	//
	//	CHAOSPERF_TEST(MyPerfTests, TestSomethingsPerf)
	//	{
	//		for (int32 Index = 0; Index < 1000000; ++Index)
	//		{
	//			static int32 Count = 0;
	//			++Count;
	//		}
	//	}
	//
	// This will create a custom class for the test derived from FPerfTest
	// and the code in the braces following the macro is the implementation
	// of RunTest().
	// If you also need custom setup and shutdown, see CHAOSPERF_TEST_CUSTOM
	#define CHAOSPERF_TEST_BASIC(CAT_ID, TEST_ID) CHAOSPERF_TEST_BASE(FPerfTest, CAT_ID ## TEST_ID)

	// 
	#define CHAOSPERF_TEST_CUSTOM(TEST_BASE, CAT_ID, TEST_ID) CHAOSPERF_TEST_BASE(TEST_BASE, CAT_ID ## TEST_ID)
}