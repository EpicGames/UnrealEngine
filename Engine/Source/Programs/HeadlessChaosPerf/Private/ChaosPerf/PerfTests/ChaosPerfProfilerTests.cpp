// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosPerf/ChaosPerf.h"

namespace ChaosPerf
{

	// Test that the profiling fixture works in the simple use case
	CHAOSPERF_TEST_BASIC(ChaosPerf, TestCsvProfiling)
	{
		for (int32 Index = 0; Index < 1000000; ++Index)
		{
			static int32 Count = 0;
			++Count;
		}
	}


}