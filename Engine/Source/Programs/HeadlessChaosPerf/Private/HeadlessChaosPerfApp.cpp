// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosPerf/ChaosPerf.h"
#include "Modules/ModuleManager.h"
#include "RequiredProgramMainCPPInclude.h"

IMPLEMENT_APPLICATION(HeadlessChaosPerf, "HeadlessChaosPerf");

#define LOCTEXT_NAMESPACE "HeadlessChaosPerf"

DEFINE_LOG_CATEGORY(LogChaosPerf);

INT32_MAIN_INT32_ARGC_TCHAR_ARGV()
{
	// start up the main loop
	GEngineLoop.PreInit(ArgC, ArgV);
	FModuleManager::Get().StartProcessingNewlyLoadedObjects();


	ChaosPerf::FPerfTestRegistry::Get().RunAll();


	FEngineLoop::AppPreExit();
	FModuleManager::Get().UnloadModulesAtShutdown();
	FEngineLoop::AppExit();

	FPlatformMisc::RequestExit(false);

	return 0;
}

#undef LOCTEXT_NAMESPACE
